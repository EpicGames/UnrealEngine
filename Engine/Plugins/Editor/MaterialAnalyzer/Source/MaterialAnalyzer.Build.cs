// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MaterialAnalyzer : ModuleRules
	{
		public MaterialAnalyzer(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePathModuleNames.AddRange(
				new string[] {
				"WorkspaceMenuStructure"
				}
			);
			PublicDependencyModuleNames.AddRange(new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"Slate",
					"SlateCore",
					"EditorFramework",
					"UnrealEd",
					"PropertyEditor",
					"DesktopPlatform",
					"ToolWidgets",
			});

            PrivateDependencyModuleNames.AddRange(new string[] {
                    "AssetManagerEditor"
            });
        }
	}

}