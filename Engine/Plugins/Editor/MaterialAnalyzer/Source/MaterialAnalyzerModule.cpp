// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialAnalyzerModule.h"
#include "SMaterialAnalyzer.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "MaterialAnalyzer"

static const FName MaterialAnalyzerName("MaterialAnalyzer");

class FMaterialAnalyzerModule : public IModuleInterface
{
public:
	FMaterialAnalyzerModule()
	{
	}

	// FModuleInterface overrides
	virtual void StartupModule() override;
	virtual void ShutdownModule() override {}
	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}

	TSharedRef<SDockTab> SpawnMaterialAnalyzerTab(const FSpawnTabArgs& SpawnTabArgs);
};

void FMaterialAnalyzerModule::StartupModule()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
		MaterialAnalyzerName,
		FOnSpawnTab::CreateRaw(this, &FMaterialAnalyzerModule::SpawnMaterialAnalyzerTab))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsAuditCategory())
		.SetDisplayName(LOCTEXT("TabTitle", "Material Analyzer"))
		.SetTooltipText(LOCTEXT("TooltipText", "Opens Material Analyzer tool."))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "MaterialEditor.ToggleMaterialStats.Tab"));
}

TSharedRef<SDockTab> FMaterialAnalyzerModule::SpawnMaterialAnalyzerTab(const FSpawnTabArgs& SpawnTabArgs)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	TSharedPtr<SWidget> TabContent;

	TabContent = SNew(SMaterialAnalyzer, MajorTab, SpawnTabArgs.GetOwnerWindow());
	
	MajorTab->SetContent(TabContent.ToSharedRef());

	return MajorTab;
}

IMPLEMENT_MODULE(FMaterialAnalyzerModule, MaterialAnalyzer)

#undef LOCTEXT_NAMESPACE
