// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"

struct FBasePropertyOverrideNode
{
public:
	FBasePropertyOverrideNode(FName InParameterName, FName InParameterID, float InParameterValue, bool bInOverride, const FText& InHighlightText = FText::GetEmpty()) :
		ParameterName(InParameterName),
		ParameterID(InParameterID),
		ParameterValue(InParameterValue),
		bOverride(bInOverride),
		HighlightText(InHighlightText)
	{

	}
	FName ParameterName;
	FName ParameterID;
	float ParameterValue;
	bool bOverride;
	FText HighlightText;

	TArray<TSharedRef<FBasePropertyOverrideNode>>* Children;
};

struct FStaticMaterialLayerParameterNode
{
public:
	FStaticMaterialLayerParameterNode(FName InParameterName, FString InParameterValue, bool bInOverride, const FText& InHighlightText = FText::GetEmpty()):
		ParameterName(InParameterName),
		ParameterValue(InParameterValue),
		bOverride(bInOverride),
		HighlightText(InHighlightText)
	{

	}
	FName ParameterName;
	FString ParameterValue;
	bool bOverride;
	FText HighlightText;
};

struct FStaticSwitchParameterNode
{
public:
	FStaticSwitchParameterNode(FName InParameterName, bool InParameterValue, bool bInOverride, const FText& InHighlightText = FText::GetEmpty()) :
		ParameterName(InParameterName),
		ParameterValue(InParameterValue),
		bOverride(bInOverride),
		HighlightText(InHighlightText)
	{

	}
	FName ParameterName;
	bool ParameterValue;
	bool bOverride;
	FText HighlightText;

	TArray<TSharedRef<FStaticSwitchParameterNode>>* Children;
};

struct FStaticComponentMaskParameterNode
{
public:
	FStaticComponentMaskParameterNode(FName InParameterName, bool InR, bool InG, bool InB, bool InA, bool bInOverride, const FText& InHighlightText = FText::GetEmpty()) :
		ParameterName(InParameterName),
		R(InR),
		G(InG),
		B(InB),
		A(InA),
		bOverride(bInOverride),
		HighlightText(InHighlightText)
	{

	}
	FName ParameterName;
	bool R;
	bool G;
	bool B;
	bool A;
	bool bOverride;
	FText HighlightText;
};

typedef TSharedRef<FBasePropertyOverrideNode, ESPMode::ThreadSafe> FBasePropertyOverrideNodeRef;

typedef TSharedRef<FStaticMaterialLayerParameterNode, ESPMode::ThreadSafe> FStaticMaterialLayerParameterNodeRef;

typedef TSharedRef<FStaticSwitchParameterNode, ESPMode::ThreadSafe> FStaticSwitchParameterNodeRef;

typedef TSharedRef<FStaticComponentMaskParameterNode, ESPMode::ThreadSafe> FStaticComponentMaskParameterNodeRef;

typedef TSharedRef<struct FAnalyzedMaterialNode, ESPMode::ThreadSafe> FAnalyzedMaterialNodeRef;

typedef TSharedPtr<struct FAnalyzedMaterialNode, ESPMode::ThreadSafe> FAnalyzedMaterialNodePtr;

struct FAnalyzedMaterialNode
{
public:
	/**
	* Add the given node to our list of children for this material (this node will keep a strong reference to the instance)
	*/
	FAnalyzedMaterialNodeRef* AddChildNode(FAnalyzedMaterialNodeRef InChildNode)
	{
		ChildNodes.Add(InChildNode);
		return &ChildNodes.Last();
	}

	/**
	* @return The node entries for the material's children
	*/
	TArray<FAnalyzedMaterialNodeRef>& GetChildNodes()
	{
		return ChildNodes;
	}

	TArray<FAnalyzedMaterialNodeRef>* GetChildNodesPtr()
	{
		return &ChildNodes;
	}

	int32 ActualNumberOfChildren() const
	{
		return ChildNodes.Num();
	}

	int32 TotalNumberOfChildren() const
	{
		int32 TotalChildren = 0;

		for(const FAnalyzedMaterialNodeRef& ChildNode : ChildNodes)
		{
			TotalChildren += ChildNode->TotalNumberOfChildren();
		}

		return TotalChildren + ChildNodes.Num();
	}

	FBasePropertyOverrideNodeRef FindBasePropertyOverride(FName ParameterName)
	{
		FBasePropertyOverrideNodeRef* BasePropertyOverride = BasePropertyOverrides.FindByPredicate([&](const FBasePropertyOverrideNodeRef& Entry) { return Entry->ParameterName == ParameterName; });
		check(BasePropertyOverride != nullptr);
		return *BasePropertyOverride;
	}

	FStaticMaterialLayerParameterNodeRef FindMaterialLayerParameter(FName ParameterName)
	{
		FStaticMaterialLayerParameterNodeRef* MaterialLayerParameter = MaterialLayerParameters.FindByPredicate([&](const FStaticMaterialLayerParameterNodeRef& Entry) { return Entry->ParameterName == ParameterName; });
		check(MaterialLayerParameter != nullptr);
		return *MaterialLayerParameter;
	}

	FStaticSwitchParameterNodeRef FindStaticSwitchParameter(FName ParameterName)
	{
		FStaticSwitchParameterNodeRef* StaticSwitchParameter = StaticSwitchParameters.FindByPredicate([&](const FStaticSwitchParameterNodeRef& Entry) { return Entry->ParameterName == ParameterName; });
		check(StaticSwitchParameter != nullptr);
		return *StaticSwitchParameter;
	}

	FStaticComponentMaskParameterNodeRef FindStaticComponentMaskParameter(FName ParameterName)
	{
		FStaticComponentMaskParameterNodeRef* StaticComponentMaskParameter = StaticComponentMaskParameters.FindByPredicate([&](const FStaticComponentMaskParameterNodeRef& Entry) { return Entry->ParameterName == ParameterName; });
		check(StaticComponentMaskParameter != nullptr);
		return *StaticComponentMaskParameter;
	}

	bool HasAnyFilteredParameters(const FString& ParameterFilter) const
	{
		// Check if this material node has any filtered parameters
		const bool bHasAnyFilteredBaseProperties = BasePropertyOverrides.FindByPredicate(
			[&](const FBasePropertyOverrideNodeRef& Entry) -> bool
			{
				// Only overridden parameters are displayed, so ignore any inherited parameters for this search
				return Entry->bOverride && Entry->ParameterName.ToString().Contains(ParameterFilter);
			}) != nullptr;
		if (bHasAnyFilteredBaseProperties)
		{
			return true;
		}

		const bool bHasAnyFilteredStaticSwitchParameters = StaticSwitchParameters.FindByPredicate(
			[&](const FStaticSwitchParameterNodeRef& Entry) -> bool
			{
				// Only overridden parameters are displayed, so ignore any inherited parameters for this search
				return Entry->bOverride && Entry->ParameterName.ToString().Contains(ParameterFilter);
			}) != nullptr;
		if (bHasAnyFilteredStaticSwitchParameters)
		{
			return true;
		}

		const bool bHasAnyFilteredStaticComponentMaskParameters = StaticComponentMaskParameters.FindByPredicate(
			[&](const FStaticComponentMaskParameterNodeRef& Entry) -> bool
			{
				// Only overridden parameters are displayed, so ignore any inherited parameters for this search
				return Entry->bOverride && Entry->ParameterName.ToString().Contains(ParameterFilter);
			}) != nullptr;
		if (bHasAnyFilteredStaticComponentMaskParameters)
		{
			return true;
		}

		// Now check child nodes recursively
		for (const FAnalyzedMaterialNodeRef& Child : ChildNodes)
		{
			if (Child->HasAnyFilteredParameters(ParameterFilter))
			{
				return true;
			}
		}
		return false;
	}

	FString Path;
	FSoftObjectPath ObjectPath;
	FAnalyzedMaterialNodePtr Parent;
	FAssetData AssetData;

	TArray<FBasePropertyOverrideNodeRef> BasePropertyOverrides;
	TArray<FStaticMaterialLayerParameterNodeRef> MaterialLayerParameters;
	TArray<FStaticSwitchParameterNodeRef> StaticSwitchParameters;
	TArray<FStaticComponentMaskParameterNodeRef> StaticComponentMaskParameters;

protected:
	TArray<FAnalyzedMaterialNodeRef> ChildNodes;
};
