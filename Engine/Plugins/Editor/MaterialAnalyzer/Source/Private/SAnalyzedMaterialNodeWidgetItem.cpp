// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnalyzedMaterialNodeWidgetItem.h"
#include "MaterialShaderType.h"
#include "Editor.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "MaterialAnalyzer"

FName SAnalyzedMaterialNodeWidgetItem::NAME_MaterialName(TEXT("MaterialName"));
FName SAnalyzedMaterialNodeWidgetItem::NAME_NumberOfChildren(TEXT("MaterialChildren"));
FName SAnalyzedMaterialNodeWidgetItem::NAME_BasePropertyOverrides(TEXT("BasePropertyOverrides"));
FName SAnalyzedMaterialNodeWidgetItem::NAME_MaterialLayerParameters(TEXT("MaterialLayerParameters"));
FName SAnalyzedMaterialNodeWidgetItem::NAME_StaticSwitchParameters(TEXT("StaticSwitchParameters"));
FName SAnalyzedMaterialNodeWidgetItem::NAME_StaticComponentMaskParameters(TEXT("StaticComponentMaskParameters"));

void SAnalyzedMaterialNodeWidgetItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	MaterialInfo = InArgs._MaterialInfoToVisualize;

	SetPadding(0);
	BasePropertyOverrideNodes = MaterialInfo->BasePropertyOverrides;
	StaticSwitchNodes = MaterialInfo->StaticSwitchParameters;
	StaticMaterialLayerNodes = MaterialInfo->MaterialLayerParameters;
	StaticComponentMaskNodes = MaterialInfo->StaticComponentMaskParameters;
	AssetData = MaterialInfo->AssetData;
	CachedMaterialName = FText::FromString(MaterialInfo->Path);
	TotalNumberOfChildren = MaterialInfo->TotalNumberOfChildren();
	NumberOfChildren = MaterialInfo->ActualNumberOfChildren();

	SMultiColumnTableRow< FAnalyzedMaterialNodeRef >::Construct(SMultiColumnTableRow< FAnalyzedMaterialNodeRef >::FArguments()
		.Padding(0)
		.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow")
		, InOwnerTableView);
}

TSharedRef<SWidget> SAnalyzedMaterialNodeWidgetItem::GenerateWidgetForColumn(const FName& ColumnName)
{
	if(ColumnName == NAME_MaterialName)
	{
		return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SExpanderArrow, SharedThis(this))
			.IndentAmount(16)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(this, &SAnalyzedMaterialNodeWidgetItem::GetMaterialName)
		]
		+ SHorizontalBox::Slot()
		.Padding(1.0f, 1.0f)
		.AutoWidth()
		[
			SNew(SButton)
			.OnClicked(this, &SAnalyzedMaterialNodeWidgetItem::FindInContentBrowser)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ToolTipText(LOCTEXT("BrowseInContentBrowser", "Browse to this asset in the Content Browser"))
			.DesiredSizeScale(FVector2D(0.4f, 0.4f))
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser").GetIcon())
			]
		];
	}
	else if(ColumnName == NAME_NumberOfChildren)
	{
		return SNew(STextBlock)
			.Text(this, &SAnalyzedMaterialNodeWidgetItem::GetNumberOfChildren)
			.Justification(ETextJustify::Left);
	}
	else if (ColumnName == NAME_BasePropertyOverrides)
	{
		return SNew(SBasePropertyOverrideWidget)
			.StaticInfos(BasePropertyOverrideNodes);
	}
	else if(ColumnName == NAME_MaterialLayerParameters)
	{
		return SNew(SStaticMaterialLayerParameterWidget)
			.StaticInfos(StaticMaterialLayerNodes);
	}
	else if (ColumnName == NAME_StaticSwitchParameters)
	{
		return SNew(SStaticSwitchParameterWidget)
			.StaticInfos(StaticSwitchNodes);
	}
	else if (ColumnName == NAME_StaticComponentMaskParameters)
	{
		return SNew(SStaticComponentMaskParameterWidget)
			.StaticInfos(StaticComponentMaskNodes);
	}

	return SNullWidget::NullWidget;
}

FReply SAnalyzedMaterialNodeWidgetItem::FindInContentBrowser()
{
	TArray<FAssetData> AssetDataArray;
	AssetDataArray.Add(AssetData);
	GEditor->SyncBrowserToObjects(AssetDataArray);
	return FReply::Handled();
}



template <typename NodeType>
void SStaticParameterWidget<NodeType>::Construct(const FArguments& InArgs)
{
	StaticNodes = InArgs._StaticInfos;
	StyleSet = InArgs._StyleSet;

	DataVerticalBox = SNew(SVerticalBox).Visibility(EVisibility::Collapsed);

	for (int i = 0; i < StaticNodes.Num(); ++i)
	{
		if (!StaticNodes[i]->bOverride)
		{
			continue;
		}

		DataVerticalBox->AddSlot()
		[
			CreateRowWidget(StaticNodes[i])
		];
	}

	ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SAssignNew(ExpanderButton, SButton)
						.ButtonStyle(FCoreStyle::Get(), "NoBorder")
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.ClickMethod(EButtonClickMethod::MouseDown)
						.OnClicked(this, &SStaticParameterWidget<NodeType>::DoExpand)
						.ContentPadding(0.f)
						.ForegroundColor(FSlateColor::UseForeground())
						.IsFocusable(false)
						[
							SNew(SImage)
							.Image(this, &SStaticParameterWidget <NodeType>::GetExpanderImage)
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
				]
				+ SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.Text(this, &SStaticParameterWidget<NodeType>::GetBaseText)
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				DataVerticalBox.ToSharedRef()
			]
		];

	ExpanderButton->SetVisibility(DataVerticalBox->NumSlots() > 0 ? EVisibility::Visible : EVisibility::Hidden);
}

template <typename NodeType>
FText SStaticParameterWidget<NodeType>::GetBaseText() const
{
	return LOCTEXT("NotOverridenErrorMessage", "Override SStaticParameterWidget::GetBaseText");
}


FText SBasePropertyOverrideWidget::GetBaseText() const
{
	return FText::Format(FTextFormat(LOCTEXT("NumberOfBasePropertyOverrides", "{0} Base Property Overrides")), DataVerticalBox->NumSlots());
}

TSharedRef<SWidget> SBasePropertyOverrideWidget::CreateRowWidget(FBasePropertyOverrideNodeRef RowData)
{
	FText DisplayText = FText::GetEmpty();

	if (RowData->ParameterID.IsEqual(TEXT("bOverride_OpacityMaskClipValue")))
	{
		DisplayText = FText::AsNumber(RowData->ParameterValue);
	}
	else if (RowData->ParameterID.IsEqual(TEXT("bOverride_BlendMode")))
	{
		int32 BlendID = (int32)RowData->ParameterValue;
		DisplayText = FText::FromString(GetBlendModeString((EBlendMode)BlendID));
	}
	else if (RowData->ParameterID.IsEqual(TEXT("bOverride_ShadingModel")))
	{
		int32 BlendID = (int32)RowData->ParameterValue;
		DisplayText = FText::FromString(GetShadingModelString((EMaterialShadingModel)BlendID));
	}	
	else // bool values
	{
		DisplayText = RowData->ParameterValue ? LOCTEXT("True", "True") : LOCTEXT("False", "False");
	}


	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(24, 0, 0, 0)
		[
			SNew(STextBlock)
			.Text(FText::FromName(RowData->ParameterName))
			.HighlightText(RowData->HighlightText)
		]
	+ SHorizontalBox::Slot()
		.Padding(2, 0, 0, 0)
		.AutoWidth()
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Text(DisplayText)
		];
}

FText SStaticSwitchParameterWidget::GetBaseText() const
{
	return FText::Format(FTextFormat(LOCTEXT("NumberOfStaticSwitchParameters", "{0} Static Switch Parameters")), DataVerticalBox->NumSlots());
}

TSharedRef<SWidget> SStaticSwitchParameterWidget::CreateRowWidget(FStaticSwitchParameterNodeRef RowData)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(24, 0, 0, 0)
		[
			SNew(STextBlock)
			.Text(FText::FromName(RowData->ParameterName))
			.HighlightText(RowData->HighlightText)
		]
	+ SHorizontalBox::Slot()
		.Padding(2, 0, 0, 0)
		.AutoWidth()
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Text(RowData->ParameterValue ? LOCTEXT("True", "True") : LOCTEXT("False", "False"))
		];
}

FText SStaticComponentMaskParameterWidget::GetBaseText() const
{
	return FText::Format(FTextFormat(LOCTEXT("NumberOfStaticComponentMaskParameters", "{0} Static Component Mask Parameters")), DataVerticalBox->NumSlots());
}


TSharedRef<SWidget> SStaticComponentMaskParameterWidget::CreateRowWidget(FStaticComponentMaskParameterNodeRef RowData)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(24, 0, 0, 0)
		[
			SNew(STextBlock)
			.Text(FText::FromName(RowData->ParameterName))
			.HighlightText(RowData->HighlightText)
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 10, 0)
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("R")))
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(RowData->R ? LOCTEXT("True", "True") : LOCTEXT("False", "False"))
				]
			]
			+ SVerticalBox::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 10, 0)
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("G")))
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(RowData->G ? LOCTEXT("True", "True") : LOCTEXT("False", "False"))
				]
			]
			+ SVerticalBox::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 10, 0)
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("B")))
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(RowData->B ? LOCTEXT("True", "True") : LOCTEXT("False", "False"))
				]
			]
			+ SVerticalBox::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0,0,10,0)
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("A")))
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(RowData->A ? LOCTEXT("True", "True") : LOCTEXT("False", "False"))
				]
			]
		];
}

FText SStaticMaterialLayerParameterWidget::GetBaseText() const
{
	return FText::Format(FTextFormat(LOCTEXT("NumberOfStaticMaterialLayerParameters", "{0} Static Material Layer Parameters")), DataVerticalBox->NumSlots());
}

TSharedRef<SWidget> SStaticMaterialLayerParameterWidget::CreateRowWidget(FStaticMaterialLayerParameterNodeRef RowData)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(24, 0, 0, 0)
		[
			SNew(STextBlock)
			.Text(FText::FromName(RowData->ParameterName))
			.HighlightText(RowData->HighlightText)
		]
	+ SHorizontalBox::Slot()
		.Padding(2, 0, 0, 0)
		[
			SNew(STextBlock)
			.Text(FText::FromString(RowData->ParameterValue))
		];

}

template <typename NodeType>
FReply SStaticParameterWidget<NodeType>::DoExpand()
{
	if(bIsExpanded)
	{
		DataVerticalBox->SetVisibility(EVisibility::Collapsed);
		bIsExpanded = false;
	}
	else
	{
		DataVerticalBox->SetVisibility(EVisibility::Visible);
		bIsExpanded = true;
	}
	return FReply::Handled();
}

/** @return the name of an image that should be shown as the expander arrow */
template <typename NodeType>
const FSlateBrush* SStaticParameterWidget<NodeType>::GetExpanderImage() const
{
	FName ResourceName;
	if (bIsExpanded)
	{
		if (ExpanderButton->IsHovered())
		{
			static FName ExpandedHoveredName = "TreeArrow_Expanded_Hovered";
			ResourceName = ExpandedHoveredName;
		}
		else
		{
			static FName ExpandedName = "TreeArrow_Expanded";
			ResourceName = ExpandedName;
		}
	}
	else
	{
		if (ExpanderButton->IsHovered())
		{
			static FName CollapsedHoveredName = "TreeArrow_Collapsed_Hovered";
			ResourceName = CollapsedHoveredName;
		}
		else
		{
			static FName CollapsedName = "TreeArrow_Collapsed";
			ResourceName = CollapsedName;
		}
	}

	return StyleSet->GetBrush(ResourceName);
}


#undef LOCTEXT_NAMESPACE
