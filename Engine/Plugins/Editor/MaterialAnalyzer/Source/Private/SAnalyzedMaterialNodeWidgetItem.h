// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "AnalyzedMaterialNode.h"

class STableViewBase;
class SVerticalBox;

class SAnalyzedMaterialNodeWidgetItem
	: public SMultiColumnTableRow<FAnalyzedMaterialNodeRef>
{
public:
	static FName NAME_MaterialName;
	static FName NAME_NumberOfChildren;
	static FName NAME_BasePropertyOverrides;
	static FName NAME_MaterialLayerParameters;
	static FName NAME_StaticSwitchParameters;
	static FName NAME_StaticComponentMaskParameters;

	SLATE_BEGIN_ARGS(SAnalyzedMaterialNodeWidgetItem)
		: _MaterialInfoToVisualize()
	{ }

		SLATE_ARGUMENT(FAnalyzedMaterialNodePtr, MaterialInfoToVisualize)

	SLATE_END_ARGS()


public:

	/**
	* Construct child widgets that comprise this widget.
	*
	* @param InArgs Declaration from which to construct this widget.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);


	// SMultiColumnTableRow overrides
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	FReply FindInContentBrowser();
	FText GetMaterialName() const
	{
		return CachedMaterialName;
	}

	FText GetNumberOfChildren() const
	{
		return FText::Format(FTextFormat::FromString(TEXT("{0}/{1}")), NumberOfChildren, TotalNumberOfChildren);
	}

protected:
	/** The info about the widget that we are visualizing. */
	FAnalyzedMaterialNodePtr MaterialInfo;

	FText CachedMaterialName;
	int TotalNumberOfChildren;
	int NumberOfChildren;
	FAssetData AssetData;
	TArray<FBasePropertyOverrideNodeRef> BasePropertyOverrideNodes;
	TArray<FStaticSwitchParameterNodeRef> StaticSwitchNodes;
	TArray<FStaticComponentMaskParameterNodeRef> StaticComponentMaskNodes;
	TArray<FStaticMaterialLayerParameterNodeRef> StaticMaterialLayerNodes;
};

template<typename NodeType>
class SStaticParameterWidget
	: public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SStaticParameterWidget)
		: _StyleSet(&FCoreStyle::Get()),
		_StaticInfos()
	{}

		SLATE_ARGUMENT(const ISlateStyle*, StyleSet)
		SLATE_ARGUMENT(TArray<NodeType>, StaticInfos)

	SLATE_END_ARGS()
public:
	SStaticParameterWidget() :
		bIsExpanded(false)
	{

	}

	virtual TSharedRef<SWidget> CreateRowWidget(NodeType RowData)
	{
		return SNullWidget::NullWidget;
	}

	virtual FText GetBaseText() const;

	void Construct(const FArguments& InArgs);

	TSharedPtr<SVerticalBox> DataVerticalBox;
	TSharedPtr<SButton> ExpanderButton;

	TArray<NodeType> StaticNodes;

	FReply DoExpand();
	const FSlateBrush* GetExpanderImage() const;

	/** The slate style to use */
	const ISlateStyle* StyleSet;

	bool bIsExpanded;

};

class SBasePropertyOverrideWidget
	: public SStaticParameterWidget<FBasePropertyOverrideNodeRef>
{
	virtual TSharedRef<SWidget> CreateRowWidget(FBasePropertyOverrideNodeRef RowData) override;

	virtual FText GetBaseText() const override;
};

class SStaticSwitchParameterWidget
	: public SStaticParameterWidget<FStaticSwitchParameterNodeRef>
{
	virtual TSharedRef<SWidget> CreateRowWidget(FStaticSwitchParameterNodeRef RowData) override;

	virtual FText GetBaseText() const override;
};

class SStaticComponentMaskParameterWidget
	: public SStaticParameterWidget<FStaticComponentMaskParameterNodeRef>
{
	virtual TSharedRef<SWidget> CreateRowWidget(FStaticComponentMaskParameterNodeRef RowData) override;

	virtual FText GetBaseText() const override;
};

class SStaticMaterialLayerParameterWidget
	: public SStaticParameterWidget<FStaticMaterialLayerParameterNodeRef>
{
public:
	virtual TSharedRef<SWidget> CreateRowWidget(FStaticMaterialLayerParameterNodeRef RowData) override;
	virtual FText GetBaseText() const override;
};
