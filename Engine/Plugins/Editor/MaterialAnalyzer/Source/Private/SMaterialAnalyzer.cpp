// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMaterialAnalyzer.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ContentBrowserDataLegacyBridge.h"
#include "PropertyCustomizationHelpers.h"
#include "CollectionManagerModule.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Framework/Application/SlateApplication.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "ICollectionSource.h"
#include "AssetManagerEditorModule.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "SAnalyzedMaterialNodeWidgetItem.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/STreeView.h"
#include "Filters/SFilterSearchBox.h"
#include "Styling/StyleColors.h"
#include "DesktopPlatformModule.h"
#include "ProfilingDebugging/DiagnosticTable.h"

#define LOCTEXT_NAMESPACE "MaterialAnalyzer"

static TMap<FName, FName> BasePropertyOverrideNames;

SMaterialAnalyzer::SMaterialAnalyzer()
	: BuildBaseMaterialTreeTask(nullptr)
	, AnalyzeTreeTask(nullptr)
	, AnalyzeForIdenticalPermutationsTask(nullptr)
	, bWaitingForAssetRegistryLoad(false)
{
	BasePropertyOverrideNames.Empty();
	BasePropertyOverrideNames.Add(TEXT("bOverride_OpacityMaskClipValue"), TEXT("OpacityMaskClipValueOverride"));
	BasePropertyOverrideNames.Add(TEXT("bOverride_BlendMode"), TEXT("BlendModeOverride"));
	BasePropertyOverrideNames.Add(TEXT("bOverride_ShadingModel"), TEXT("ShadingModelOverride"));
	BasePropertyOverrideNames.Add(TEXT("bOverride_DitheredLODTransition"), TEXT("DitheredLODTransitionOverride"));
	BasePropertyOverrideNames.Add(TEXT("bOverride_CastDynamicShadowAsMasked"), TEXT("CastDynamicShadowAsMaskedOverride"));
	BasePropertyOverrideNames.Add(TEXT("bOverride_TwoSided"), TEXT("TwoSidedOverride"));
	BasePropertyOverrideNames.Add(TEXT("bOverride_bIsThinSurface"), TEXT("bIsThinSurfaceOverride"));
	BasePropertyOverrideNames.Add(TEXT("bOverride_OutputTranslucentVelocity"), TEXT("bOutputTranslucentVelocity"));
	BasePropertyOverrideNames.Add(TEXT("bOverride_bHasPixelAnimation"), TEXT("bHasPixelAnimation"));
	BasePropertyOverrideNames.Add(TEXT("bOverride_bEnableTessellation"), TEXT("bEnableTessellation"));
}

SMaterialAnalyzer::~SMaterialAnalyzer()
{

}

const FAssetData* FindParentAssetData(const FAssetData* InAssetData, const TArray<FAssetData>& ArrayToSearch)
{
	check(InAssetData != nullptr);
	static const FName NAME_Parent = TEXT("Parent");
	FString ParentPathString = InAssetData->GetTagValueRef<FString>(NAME_Parent);

	int32 FirstCut = INDEX_NONE;
	ParentPathString.FindChar(L'\'', FirstCut);

	FSoftObjectPath ParentPath;

	if(FirstCut != INDEX_NONE)
	{
		ParentPath = ParentPathString.Mid(FirstCut + 1, ParentPathString.Len() - FirstCut - 2);
	}
	else
	{
		ParentPath = ParentPathString;
	}

	if(ParentPath.IsValid())
	{
		return ArrayToSearch.FindByPredicate(
			[&](FAssetData& Entry)
		{
			return Entry.GetSoftObjectPath() == ParentPath;
		}
		);
	}

	return nullptr;
}

void SMaterialAnalyzer::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	TArray<const UClass*> AllowedClasses;
	AllowedClasses.Add(UMaterialInterface::StaticClass());
	
	TSharedRef<SWidget> AssetPickerWidget = SNew(SObjectPropertyEntryBox)
		.ObjectPath(this, &SMaterialAnalyzer::GetCurrentAssetPath)
		.AllowedClass(UMaterialInterface::StaticClass())
		.OnObjectChanged(this, &SMaterialAnalyzer::OnAssetSelected)
		.AllowClear(false)
		.DisplayUseSelected(true)
		.DisplayBrowse(true)
		.NewAssetFactories(TArray<UFactory*>())
		.IsEnabled(this, &SMaterialAnalyzer::IsMaterialSelectionAllowed);

	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		.BorderBackgroundColor(FLinearColor::Gray) // Darken the outer border
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(5.0f, 5.0f, 5.0f, 5.0f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MaterialToAnalyzeLabel", "Material To Analyze: "))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(0.4f)
				[
					AssetPickerWidget
				]
				+ SHorizontalBox::Slot()
				.FillWidth(0.4f)
				[
					SNew(SFilterSearchBox)
					.HintText(LOCTEXT("MaterialParametersToFilterHint", "Parameters to Filter..."))
					.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search (pressing enter selects the results)"))
					.OnTextCommitted(this, &SMaterialAnalyzer::OnParameterFilterChanged)
				]
				+SHorizontalBox::Slot()
				.FillWidth(0.2f)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "ButtonStyle")
					.OnClicked(this, &SMaterialAnalyzer::OnExportAnalyzedMaterialToCSV)
					.ContentPadding(FMargin(2.0f))
					.Content()
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.0f)
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Save"))
							.ColorAndOpacity(FSlateColor(EStyleColor::Black))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.0f)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ExportToCSV", "Export to CSV"))
							.ColorAndOpacity(FSlateColor(EStyleColor::Black))
						]
					]
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SSplitter)
				.Orientation(EOrientation::Orient_Vertical)
				+ SSplitter::Slot()
				[
					SNew(SBorder)
					.Padding(0)
					.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
					[
						SAssignNew(MaterialTree, SAnalyzedMaterialTree)
						.TreeItemsSource(&MaterialTreeRoot)
						.OnGenerateRow(this, &SMaterialAnalyzer::HandleReflectorTreeGenerateRow)
						.OnGetChildren(this, &SMaterialAnalyzer::HandleReflectorTreeGetChildren)
						.OnSetExpansionRecursive(this, &SMaterialAnalyzer::HandleReflectorTreeRecursiveExpansion)
						.HeaderRow
						(
							SNew(SHeaderRow)
							+ SHeaderRow::Column(SAnalyzedMaterialNodeWidgetItem::NAME_MaterialName)
							.DefaultLabel(LOCTEXT("MaterialNameLabel", "Material Name"))
							.FillWidth(0.80f)
							+ SHeaderRow::Column(SAnalyzedMaterialNodeWidgetItem::NAME_NumberOfChildren)
							.DefaultLabel(LOCTEXT("NumberOfMaterialChildrenLabel", "Number of Children (Direct/Total)"))
							+ SHeaderRow::Column(SAnalyzedMaterialNodeWidgetItem::NAME_BasePropertyOverrides)
							.DefaultLabel(LOCTEXT("BasePropertyOverridesLabel", "Base Property Overrides"))
							+ SHeaderRow::Column(SAnalyzedMaterialNodeWidgetItem::NAME_MaterialLayerParameters)
							.DefaultLabel(LOCTEXT("MaterialLayerParametersLabel", "Material Layer Parameters"))
							+ SHeaderRow::Column(SAnalyzedMaterialNodeWidgetItem::NAME_StaticSwitchParameters)
							.DefaultLabel(LOCTEXT("StaticSwitchParametersLabel", "Static Switch Parameters"))
							+ SHeaderRow::Column(SAnalyzedMaterialNodeWidgetItem::NAME_StaticComponentMaskParameters)
							.DefaultLabel(LOCTEXT("StaticComponenetMaskParametersLabel", "Static Component Mask Parameters"))
						)
					]
				]
				+ SSplitter::Slot()
				[
					SNew(SBorder)
					.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
					.BorderBackgroundColor(FLinearColor::Gray) // Darken the outer border
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("SuggestionsLabel", "Suggestions"))
						]
						+ SVerticalBox::Slot()
						[
							SAssignNew(SuggestionsBox, SScrollBox)
						]
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(EVerticalAlignment::VAlign_Bottom)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SAssignNew(StatusBox, STextBlock)
					.Text(LOCTEXT("DoneLabel", "Done"))
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.Padding(0,0,4,0)
				[
					SAssignNew(StatusThrobber, SThrobber)
					.Animate(SThrobber::EAnimation::None)
				]
			]
		]
	];

	// Load the asset registry module to listen for updates
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	if(AssetRegistryModule.Get().IsLoadingAssets())
	{
		StartAsyncWork(LOCTEXT("WaitingForAssetRegistry", "Waiting for Asset Registry to finish loading"));
		bWaitingForAssetRegistryLoad = true;
	}
	else
	{
		SetupAssetRegistryCallbacks();
		BuildBasicMaterialTree();
	}
}

void SMaterialAnalyzer::SetupAssetRegistryCallbacks()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnAssetAdded().AddSP(this, &SMaterialAnalyzer::OnAssetAdded);
}

void SMaterialAnalyzer::OnAssetAdded(const FAssetData& InAssetData)
{
	if(InAssetData.IsInstanceOf(UMaterialInterface::StaticClass()))
	{
		RecentlyAddedAssetData.Add(InAssetData);
	}
}

void SMaterialAnalyzer::OnAssetSelected(const FAssetData& AssetData)
{
	if (AnalyzeTreeTask == nullptr)
	{
		CurrentlySelectedAsset = AssetData;
		UpdateViewForSelectedAsset();
	}
}

void SMaterialAnalyzer::OnParameterFilterChanged(const FText& Filter, const ETextCommit::Type InTextAction)
{
	if (AnalyzeTreeTask == nullptr)
	{
		ParameterFilter = Filter;
		bHasParameterFilterChanged = true;

		// Parameter filter is applied in two steps:
		// 1. Re-build analyzed material tree in order to highlight the filtered parameters (FBuildBasicMaterialTreeAsyncTask)
		// 2. Apply filter to UI elements by re-analyzing material tree (FAnalyzeMaterialTreeAsyncTask)
		if (CurrentlySelectedAsset.IsValid())
		{
			RecentlyAddedAssetData.Add(CurrentlySelectedAsset);
		}
	}
}

void WriteAnalyzedMaterialNodeToCSVStringInternal(const FAnalyzedMaterialNodeRef& Node, FDiagnosticTableWriterCSV& CSVTable)
{
	auto AddCSVCell = [&CSVTable](int32 NumElements, const TFunction<FString(int32 ElementIndex)>& ElementNameCallback) -> void
		{
			FString Cell;
			for (int32 ElementIndex = 0; ElementIndex < NumElements; ++ElementIndex)
			{
				Cell += ElementNameCallback(ElementIndex);
				if (ElementIndex + 1 < NumElements)
				{
					Cell += TEXT("\n");
				}
			}
			CSVTable.AddColumn(TEXT("%s"), *Cell);
		};

	FString Output;

	CSVTable.AddColumn(TEXT("%s"), *Node->AssetData.AssetName.ToString());

	AddCSVCell(
		Node->BasePropertyOverrides.Num(),
		[Node](int32 ElementIndex)
		{
			return FString::Printf(TEXT("%s ( %f )"), *Node->BasePropertyOverrides[ElementIndex]->ParameterName.ToString(), Node->BasePropertyOverrides[ElementIndex]->ParameterValue);
		}
	);
	AddCSVCell(
		Node->MaterialLayerParameters.Num(),
		[Node](int32 ElementIndex)
		{
			return FString::Printf(TEXT("%s ( %s )"), *Node->MaterialLayerParameters[ElementIndex]->ParameterName.ToString(), *Node->MaterialLayerParameters[ElementIndex]->ParameterValue);
		}
	);
	AddCSVCell(
		Node->StaticSwitchParameters.Num(),
		[Node](int32 ElementIndex)
		{
			return FString::Printf(TEXT("%s ( %s )"), *Node->StaticSwitchParameters[ElementIndex]->ParameterName.ToString(), Node->StaticSwitchParameters[ElementIndex]->ParameterValue ? TEXT("True") : TEXT("False"));
		}
	);
	AddCSVCell(
		Node->StaticComponentMaskParameters.Num(),
		[Node](int32 ElementIndex)
		{
			const FStaticComponentMaskParameterNode& Parameter = *Node->StaticComponentMaskParameters[ElementIndex];
			return FString::Printf(
				TEXT("%s ( %s%s%s%s )"),
				*Parameter.ParameterName.ToString(),
				Parameter.R ? TEXT("R") : TEXT("_"),
				Parameter.G ? TEXT("G") : TEXT("_"),
				Parameter.B ? TEXT("B") : TEXT("_"),
				Parameter.A ? TEXT("A") : TEXT("_")
			);
		}
	);

	CSVTable.CycleRow();

	for (const FAnalyzedMaterialNodeRef& ChildNode : Node->GetChildNodes())
	{
		WriteAnalyzedMaterialNodeToCSVStringInternal(ChildNode, CSVTable);
	}
}

static void WriteAnalyzedMaterialNodeToCSVString(const FAnalyzedMaterialNodeRef& Node, FDiagnosticTableWriterCSV& CSVTable)
{
	CSVTable.AddColumn(TEXT("MATERIAL"));
	CSVTable.AddColumn(TEXT("BASE PROPERTY OVERRIDES"));
	CSVTable.AddColumn(TEXT("LAYER PARAMETERS"));
	CSVTable.AddColumn(TEXT("STATIC SWITCHES"));
	CSVTable.AddColumn(TEXT("STATIC COMPONENT MASKS"));
	CSVTable.CycleRow();

	WriteAnalyzedMaterialNodeToCSVStringInternal(Node, CSVTable);
}

static bool SaveFileDialog(const FString& Title, const FString& FileTypes, FString& OutFilename, FString& InOutLastFileanme)
{
	OutFilename.Empty();

	TArray<FString> OutFilenames;
	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		const bool bFileChosen = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			Title,
			FPaths::GetPath(InOutLastFileanme),
			FPaths::GetPathLeaf(InOutLastFileanme),
			FileTypes,
			EFileDialogFlags::None,
			OutFilenames
		);
		if (bFileChosen && OutFilenames.Num() > 0)
		{
			// User successfully chose a file; remember the path for the next time the dialog opens.
			OutFilename = OutFilenames[0];
			InOutLastFileanme = OutFilenames[0];
			return true;
		}
	}

	return false;
}

FReply SMaterialAnalyzer::OnExportAnalyzedMaterialToCSV()
{
	if (!MaterialTreeRoot.IsEmpty())
	{
		FString ExportFilename;
		static FString LastUsedFilename = FPaths::Combine(FPaths::GetProjectFilePath(), TEXT("Saved"), TEXT("Logs"), TEXT("MaterialProperties.csv"));
		if (SaveFileDialog(NSLOCTEXT("UnrealEd", "Export", "Export").ToString(), TEXT("Comma Separated Value (CSV) Files|*.csv"), ExportFilename, LastUsedFilename))
		{
			if (TUniquePtr<FArchive> CSVTableFile = TUniquePtr<FArchive>{ IFileManager::Get().CreateFileWriter(*ExportFilename) })
			{
				FDiagnosticTableWriterCSV CSVTable{ CSVTableFile.Get() };
				WriteAnalyzedMaterialNodeToCSVString(MaterialTreeRoot[0], CSVTable);
			}
			else
			{
				FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("Export operation failed!"), *NSLOCTEXT("UnrealEd", "Error", "!").ToString());
			}
		}
	}
	return FReply::Handled();
}

void SMaterialAnalyzer::UpdateViewForSelectedAsset()
{
	if (CurrentlySelectedAsset.IsValid())
	{
		const FAssetData* ParentAssetData = &CurrentlySelectedAsset;
		const FAssetData* NextParentAssetData = FindParentAssetData(&CurrentlySelectedAsset, AssetDataArray);
		// get the topmost parent
		while (NextParentAssetData != nullptr)
		{
			ParentAssetData = NextParentAssetData;
			NextParentAssetData = FindParentAssetData(ParentAssetData, AssetDataArray);
		}

		// empty the previous tree root
		MaterialTreeRoot.Empty(1);
		// Add the new tree root
		FAnalyzedMaterialNodeRef* NewRoot = AllMaterialTreeRoots.FindByPredicate([&](FAnalyzedMaterialNodeRef& Entry)
		{
			return Entry->ObjectPath == ParentAssetData->GetSoftObjectPath();
		});
		check(NewRoot != nullptr);

		MaterialTreeRoot.Add(*NewRoot);

		MaterialTree->RequestTreeRefresh();

		SuggestionsBox->ClearChildren();

		AnalyzeTreeTask = new FAsyncTask<FAnalyzeMaterialTreeAsyncTask>(*NewRoot, AssetDataArray, ParameterFilter);

		StartAsyncWork(FText::Format(LOCTEXT("AnalyzingMaterial", "Analyzing {0}"), FText::FromString(AnalyzeTreeTask->GetTask().CurrentMaterialNode->Path)));
		AnalyzeTreeTask->StartBackgroundTask();
	}
}

void SMaterialAnalyzer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bWaitingForAssetRegistryLoad)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		if(!AssetRegistryModule.Get().IsLoadingAssets())
		{
			SetupAssetRegistryCallbacks();
			BuildBasicMaterialTree();
			bWaitingForAssetRegistryLoad = false;
		}
	}
	else
	{
		if (BuildBaseMaterialTreeTask != nullptr && BuildBaseMaterialTreeTask->IsDone())
		{
			delete BuildBaseMaterialTreeTask;
			BuildBaseMaterialTreeTask = nullptr;
			AsyncWorkFinished(FText::Format(FTextFormat(LOCTEXT("DoneWithMaterialInterfaces", "Done with {0} MaterialInterfaces")), GetTotalNumberOfMaterialNodes()));

			// If the parameter filter has changed, we also have to run analysis async task again,
			// which applies the actual parameter filter to UI elements in the tree view pane.
			if (bHasParameterFilterChanged)
			{
				UpdateViewForSelectedAsset();
				bHasParameterFilterChanged = false;
			}
			else
			{
				MaterialTree->RequestTreeRefresh();
			}
		}

		if (BuildBaseMaterialTreeTask == nullptr && RecentlyAddedAssetData.Num() > 0)
		{
			// Need to make this append to the previously generated list instead of erase all of the old info
			// Current problem is that if we only have a portion of the asset registry it will create duplicate
			// nodes since it won't find all parents in the tree. Need to modify the async task to not create
			// nodes that don't have parents until we can find their parent.
			AssetDataArray.Append(RecentlyAddedAssetData);
			RecentlyAddedAssetData.Empty();
			AllMaterialTreeRoots.Empty(AllMaterialTreeRoots.Num());

			BuildBaseMaterialTreeTask = new FAsyncTask<FBuildBasicMaterialTreeAsyncTask>(AllMaterialTreeRoots, AssetDataArray);
			BuildBaseMaterialTreeTask->StartBackgroundTask();

			StartAsyncWork(LOCTEXT("BuildingBasicTree", "Building Basic MaterialTree"));
		}

		if (AnalyzeTreeTask != nullptr && AnalyzeTreeTask->IsDone())
		{
			if (AnalyzeTreeTask->GetTask().LoadNextMaterial())
			{
				StartAsyncWork(FText::Format(LOCTEXT("AnalyzingMaterial", "Analyzing {0}"), FText::FromString(AnalyzeTreeTask->GetTask().CurrentMaterialNode->Path)));
				AnalyzeTreeTask->StartBackgroundTask();
			}
			else
			{
				MaterialTree->RequestListRefresh();

				// Kick off a check for identical permutations
				// @todo make this a series of tests that users can choose to run
				AnalyzeForIdenticalPermutationsTask = new FAsyncTask<FAnalyzeForIdenticalPermutationsAsyncTask>(AnalyzeTreeTask->GetTask().MaterialTreeRoot);
				AnalyzeForIdenticalPermutationsTask->StartBackgroundTask();

				delete AnalyzeTreeTask;
				AnalyzeTreeTask = nullptr;
				
				StartAsyncWork(LOCTEXT("AnalyzingTreeForIdenticalPermutations", "Analyzing material tree for identical permutations"));
			}
		}

		if (AnalyzeForIdenticalPermutationsTask != nullptr && AnalyzeForIdenticalPermutationsTask->IsDone())
		{
			MaterialTree->RequestListRefresh();
			AsyncWorkFinished(LOCTEXT("Done", "Done!"));

			TMultiMap<int32, FPermutationSuggestionData> Suggestions = AnalyzeForIdenticalPermutationsTask->GetTask().GetSuggestions();

			Suggestions.KeySort([](int32 A, int32 B) {
				return A > B; // sort to show most improvement possibility first
			});

			

			
			int32 BackgroundColorCounter = 0;
			SuggestionDataArray.Empty();
			for (auto It = Suggestions.CreateConstIterator(); It; ++It)
			{
				TSharedPtr<FPermutationSuggestionView> SuggestionHeader = MakeShareable(new FPermutationSuggestionView());
				SuggestionHeader->Header = It.Value().Header;
				for (FString Material : It.Value().Materials)
				{
					TSharedPtr<FPermutationSuggestionView> SuggestionChild = MakeShareable(new FPermutationSuggestionView());
					SuggestionChild->Header = FText::FromString(Material);
					SuggestionHeader->Children.Add(SuggestionChild);
				}
				SuggestionDataArray.Add(SuggestionHeader);
			}

			SuggestionsBox->AddSlot()
			[
				SAssignNew(SuggestionsTree, STreeView<TSharedPtr<FPermutationSuggestionView>>)
				.TreeItemsSource(&SuggestionDataArray)
				.OnGenerateRow(this, &SMaterialAnalyzer::OnGenerateSuggestionRow)
				.OnGetChildren(this, &SMaterialAnalyzer::OnGetSuggestionChildren)
			];




			delete AnalyzeForIdenticalPermutationsTask;
			AnalyzeForIdenticalPermutationsTask = nullptr;
		}
	}
}

TSharedRef< ITableRow > SMaterialAnalyzer::OnGenerateSuggestionRow(TSharedPtr<FPermutationSuggestionView> Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	if (Item->Children.Num() > 0)
	{
		return SNew(STableRow<TSharedPtr<FPermutationSuggestionView>>, OwnerTable)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Bottom)
				[
					SNew(SEditableText)
					.IsReadOnly(true)
					.Text(Item->Header)
				]
				+ SVerticalBox::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Visibility(this, &SMaterialAnalyzer::ShouldShowAdvancedRecommendations, Item)
						.Text(LOCTEXT("PermutationRecommendation", "It is recommended that you reparent them in a way so only dynamic parameters differ."))
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNullWidget::NullWidget
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleSharpButton")
						.Visibility(this, &SMaterialAnalyzer::ShouldShowAdvancedRecommendations, Item)
						.OnClicked(this, &SMaterialAnalyzer::CreateLocalSuggestionCollectionClicked, Item)
						.ContentPadding(FMargin(2.0f))
						.Content()
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(2.0f)
							[
								SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
								.ColorAndOpacity(FSlateColor(EStyleColor::Black))
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(2.0f)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("CreateLocalCollection", "Create Local Collection"))
								.ColorAndOpacity(FSlateColor(EStyleColor::Black))
							]
						]
					]
				]
			];
	}
	return SNew(STableRow<TSharedPtr<FPermutationSuggestionView>>, OwnerTable)
		[
			SNew(SEditableText)
			.IsReadOnly(true)
			.Text(Item->Header)
		];
}

EVisibility SMaterialAnalyzer::ShouldShowAdvancedRecommendations(TSharedPtr<FPermutationSuggestionView> Item) const
{
	return SuggestionsTree->IsItemExpanded(Item) ? EVisibility::Visible : EVisibility::Collapsed;
}

void SMaterialAnalyzer::OnGetSuggestionChildren(TSharedPtr<FPermutationSuggestionView> InParent, TArray< TSharedPtr<FPermutationSuggestionView> >& OutChildren)
{
	OutChildren = InParent->Children;
}

FReply SMaterialAnalyzer::CreateLocalSuggestionCollectionClicked(TSharedPtr<FPermutationSuggestionView> InSuggestion)
{
	TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;
	FCollectionManagerModule::GetModule().Get().GetVisibleCollectionContainers(CollectionContainers);

	if (CollectionContainers.Num() == 1)
	{
		CreateLocalSuggestionCollection(*CollectionContainers[0], *InSuggestion);
	}
	else
	{
		// Create a menu to select the collection container.
		FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("MaterialAnalyzerCollectionContainersMenuHeading", "Collection Containers"));

		for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
		{
			MenuBuilder.AddMenuEntry(
				CollectionContainer->GetCollectionSource()->GetTitle(),
				TAttribute<FText>(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, CollectionContainer, InSuggestion]()
						{
							CreateLocalSuggestionCollection(*CollectionContainer, *InSuggestion);
						}),
					FCanExecuteAction::CreateLambda([CollectionContainer]()
						{
							return !CollectionContainer->IsReadOnly(ECollectionShareType::CST_Local);
						})));
		}

		MenuBuilder.EndSection();

		FWidgetPath WidgetPath;
		if (FSlateApplication::Get().GeneratePathToWidgetUnchecked(AsShared(), WidgetPath))
		{
			FSlateApplication::Get().PushMenu(
				AsShared(),
				WidgetPath,
				MenuBuilder.MakeWidget(),
				FSlateApplication::Get().GetCursorPos(),
				FPopupTransitionEffect(FPopupTransitionEffect::TopMenu)
			);
		}
	}

	return FReply::Handled();
}

void SMaterialAnalyzer::CreateLocalSuggestionCollection(ICollectionContainer& InCollectionContainer, const FPermutationSuggestionView& InSuggestion)
{
	TArray<FString> AllSelectedPackageNames;
	ECollectionShareType::Type ShareType = ECollectionShareType::CST_Local;
	for (TSharedPtr<FPermutationSuggestionView> Child : InSuggestion.Children)
	{
		AllSelectedPackageNames.Add(Child->Header.ToString());
	}

	if (AllSelectedPackageNames.Num() > 0)
	{
		FString FirstAssetString = CurrentlySelectedAsset.AssetName.ToString() + TEXT("_") + FString::FromInt(InSuggestion.Children.Num());
		FName FirstAssetName = FName(*FirstAssetString);

		InCollectionContainer.CreateUniqueCollectionName(FirstAssetName, ShareType, FirstAssetName);

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		TArray<FName> PackageNamesToAddToCollection;

		TArray<FName> PackageNameSet;
		for (FString PackageToAdd : AllSelectedPackageNames)
		{
			PackageNameSet.Add(FName(*FPaths::GetBaseFilename(*PackageToAdd, false)));
		}


		const FText MaterialAnalyzerText = LOCTEXT("MaterialAnalyzerPrefix", "MaterialAnalyzer");
		FName ParentName = FName(*MaterialAnalyzerText.ToString());
		if (!InCollectionContainer.CollectionExists(ParentName, ShareType))
		{
			InCollectionContainer.CreateCollection(ParentName, ShareType, ECollectionStorageMode::Static);
		}

		const bool bCollectionSucceeded = IAssetManagerEditorModule::Get().WriteCollection(InCollectionContainer, FirstAssetName, ShareType, PackageNameSet, true);
		if (bCollectionSucceeded)
		{
			InCollectionContainer.ReparentCollection(FirstAssetName, ShareType, ParentName, ShareType);
		}
	}
}

void SMaterialAnalyzer::StartAsyncWork(const FText& WorkText)
{
	bIsAsyncWorkInProgress = true;

	if(StatusBox.IsValid())
	{
		StatusBox->SetText(WorkText);
	}

	if(StatusThrobber.IsValid())
	{
		StatusThrobber->SetAnimate(SThrobber::Horizontal);
		StatusThrobber->SetVisibility(EVisibility::SelfHitTestInvisible);
	}
}

void SMaterialAnalyzer::AsyncWorkFinished(const FText& CompleteText)
{
	if (StatusBox.IsValid())
	{
		StatusBox->SetText(CompleteText);
	}

	if (StatusThrobber.IsValid())
	{
		StatusThrobber->SetAnimate(SThrobber::None);
		StatusThrobber->SetVisibility(EVisibility::Collapsed);
	}

	bIsAsyncWorkInProgress = false;
}

int32 SMaterialAnalyzer::GetTotalNumberOfMaterialNodes()
{
	int32 NumMaterialNodes = AllMaterialTreeRoots.Num();

	for(int i = 0; i < AllMaterialTreeRoots.Num(); ++i)
	{
		NumMaterialNodes += AllMaterialTreeRoots[i]->TotalNumberOfChildren();
	}

	return NumMaterialNodes;
}

FString SMaterialAnalyzer::GetCurrentAssetPath() const
{
	return CurrentlySelectedAsset.IsValid() ? CurrentlySelectedAsset.GetObjectPathString() : FString("");
}

void SMaterialAnalyzer::BuildBasicMaterialTree()
{
	static const FName AssetRegistryName(TEXT("AssetRegistry"));
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryName);

	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	AssetRegistry.GetAssetsByClass(UMaterial::StaticClass()->GetClassPathName(), AssetDataArray, true);
	AssetRegistry.GetAssetsByClass(UMaterialInstance::StaticClass()->GetClassPathName(), AssetDataArray, true);

	if (BuildBaseMaterialTreeTask == nullptr && AssetDataArray.Num() > 0)
	{
		AllMaterialTreeRoots.Empty(AllMaterialTreeRoots.Num());
		BuildBaseMaterialTreeTask = new FAsyncTask<FBuildBasicMaterialTreeAsyncTask>(AllMaterialTreeRoots, AssetDataArray);
		BuildBaseMaterialTreeTask->StartBackgroundTask();

		StartAsyncWork(LOCTEXT("BuildingBasicTree", "Building Basic MaterialTree"));

		if (StatusThrobber.IsValid())
		{
			StatusThrobber->SetAnimate(SThrobber::EAnimation::Horizontal);
		}
	}
}

TSharedRef<ITableRow> SMaterialAnalyzer::HandleReflectorTreeGenerateRow(FAnalyzedMaterialNodeRef InMaterialNode, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedPtr<SAnalyzedMaterialNodeWidgetItem> NewWidget = SNew(SAnalyzedMaterialNodeWidgetItem, OwnerTable)
		.MaterialInfoToVisualize(InMaterialNode);

	// if we're the base level we're going to expand right away
	if(!InMaterialNode->Parent.IsValid())
	{
		MaterialTree->SetItemExpansion(InMaterialNode, true);
	}

	return NewWidget.ToSharedRef();
}

void SMaterialAnalyzer::HandleReflectorTreeGetChildren(FAnalyzedMaterialNodeRef InMaterialNode, TArray<FAnalyzedMaterialNodeRef>& OutChildren)
{
	if (ParameterFilter.IsEmpty())
	{
		OutChildren = InMaterialNode->GetChildNodes();
	}
	else
	{
		OutChildren.Empty(InMaterialNode->GetChildNodes().Num());
		for (const FAnalyzedMaterialNodeRef& ChildNode : InMaterialNode->GetChildNodes())
		{
			if (ChildNode->HasAnyFilteredParameters(ParameterFilter.ToString()))
			{
				OutChildren.Add(ChildNode);
			}
		}
	}
}

void SMaterialAnalyzer::HandleReflectorTreeRecursiveExpansion(FAnalyzedMaterialNodeRef InTreeNode, bool bIsItemExpanded)
{
	TArray< FAnalyzedMaterialNodeRef > Children = InTreeNode->GetChildNodes();

	if (Children.Num())
	{
		MaterialTree->SetItemExpansion(InTreeNode, bIsItemExpanded);
		const bool bShouldSaveState = true;

		for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
		{
			FAnalyzedMaterialNodeRef Child = Children[ChildIndex];
			MaterialTree->SetItemExpansion(Child, bIsItemExpanded);
		}
	}
}

FAnalyzedMaterialNodePtr FBuildBasicMaterialTreeAsyncTask::FindOrMakeBranchNode(FAnalyzedMaterialNodePtr ParentNode, const FAssetData* ChildData)
{
	check(ChildData != nullptr);
	FAnalyzedMaterialNodeRef* OutNode = nullptr;

	FSoftObjectPath ChildPath = ChildData->GetSoftObjectPath();

	TArray<FAnalyzedMaterialNodeRef>& NodesToSearch = ParentNode.IsValid() ? ParentNode->GetChildNodes() : MaterialTreeRoot;

	OutNode = NodesToSearch.FindByPredicate([&](FAnalyzedMaterialNodeRef& Entry) { return Entry->ObjectPath == ChildPath; });

	if (OutNode == nullptr)
	{
		FAnalyzedMaterialNode ChildNode;
		ChildNode.Path = ChildData->AssetName.ToString();
		ChildNode.ObjectPath = ChildData->GetSoftObjectPath();
		ChildNode.Parent = ParentNode;
		ChildNode.AssetData = *ChildData;
		NodesToSearch.Add(FAnalyzedMaterialNodeRef(new FAnalyzedMaterialNode(ChildNode)));
		OutNode = &NodesToSearch[NodesToSearch.Num() - 1];
	}

	return FAnalyzedMaterialNodePtr (*OutNode);
}

void FBuildBasicMaterialTreeAsyncTask::DoWork()
{
	for(int i = 0; i < AssetDataToAnalyze.Num(); ++i)
	{
		const FAssetData& AssetData = AssetDataToAnalyze[i];

		TArray<const FAssetData*> FullBranch;

		const FAssetData* CurrentBranchNode = &AssetData;
		while(CurrentBranchNode)
		{
			FullBranch.Add(CurrentBranchNode);
			CurrentBranchNode = FindParentAssetData(CurrentBranchNode, AssetDataToAnalyze);
		}

		FAnalyzedMaterialNodePtr ParentNode = nullptr;

		for(int Depth = FullBranch.Num() - 1; Depth >= 0; --Depth)
		{
			ParentNode = FindOrMakeBranchNode(ParentNode, FullBranch[Depth]);
		}
	}
}

bool FAnalyzeMaterialTreeAsyncTask::LoadNextMaterial()
{
	if (CurrentMaterialQueueIndex < MaterialQueue.Num())
	{
		CurrentMaterialNode = MaterialQueue[CurrentMaterialQueueIndex];
		check(CurrentMaterialNode->ObjectPath.IsValid());

		CurrentMaterialInterface = FindObject<UMaterialInterface>(NULL, *CurrentMaterialNode->ObjectPath.ToString());
		if (CurrentMaterialInterface == nullptr)
		{
			CurrentMaterialInterface = LoadObject<UMaterialInterface>(NULL, *CurrentMaterialNode->ObjectPath.ToString());
			check(CurrentMaterialInterface);
		}

		return true;
	}

	return false;
}

void FAnalyzeMaterialTreeAsyncTask::DoWork()
{
	MaterialQueue.Append(CurrentMaterialNode->GetChildNodes());

	TArray<FMaterialParameterInfo> MaterialLayersParameterInfo;

	check(CurrentMaterialInterface != nullptr);

	TArray<FMaterialParameterInfo> ParameterInfo;
	TArray<FGuid> Guids;

	UMaterial* CurrentMaterial = Cast<UMaterial>(CurrentMaterialInterface);
	UMaterialInstance* CurrentMaterialInstance = Cast<UMaterialInstance>(CurrentMaterialInterface);

	bool bCanBeOverriden = CurrentMaterial != nullptr;

	if(CurrentMaterial != nullptr)
	{
		CurrentMaterial->GetAllStaticSwitchParameterInfo(StaticSwitchParameterInfo, StaticSwitchGuids);
	
		CurrentMaterial->GetAllStaticComponentMaskParameterInfo(StaticMaskParameterInfo, StaticMaskGuids);
	}
	
	CurrentMaterialNode->BasePropertyOverrides.Empty(BasePropertyOverrideNames.Num());

	const FString ParameterFilterString = this->ParameterFilter.ToString();
	auto IsIncludedInParameterFilter = [&ParameterFilterString](const FName& Name) -> bool
		{
			return ParameterFilterString.IsEmpty() || Name.ToString().Contains(ParameterFilterString);
		};

	for (const TPair<FName, FName>& BasePropertyOverrideName : BasePropertyOverrideNames)
	{
		float TempValue = 0.0f;
		bool bIsOverridden = false;

		if (!IsIncludedInParameterFilter(BasePropertyOverrideName.Value))
		{
			continue;
		}

		if (BasePropertyOverrideName.Key.IsEqual(TEXT("bOverride_OpacityMaskClipValue")))
		{
			TempValue = CurrentMaterialInterface->GetOpacityMaskClipValue();
			if (CurrentMaterialInstance)
			{
				bIsOverridden = CurrentMaterialInstance->BasePropertyOverrides.bOverride_OpacityMaskClipValue;
			}
		}
		else if (BasePropertyOverrideName.Key.IsEqual(TEXT("bOverride_BlendMode")))
		{
			TempValue = (float)CurrentMaterialInterface->GetBlendMode();
			if (CurrentMaterialInstance)
			{
				bIsOverridden = CurrentMaterialInstance->BasePropertyOverrides.bOverride_BlendMode;
			}
		}
		else if (BasePropertyOverrideName.Key.IsEqual(TEXT("bOverride_ShadingModel")))
		{
			if (CurrentMaterialInterface->IsShadingModelFromMaterialExpression())
			{
				TempValue = (float)MSM_FromMaterialExpression;
			}
			else
			{
				ensure(CurrentMaterialInterface->GetShadingModels().CountShadingModels() == 1);
				TempValue = (float)CurrentMaterialInterface->GetShadingModels().GetFirstShadingModel();
			}

			if (CurrentMaterialInstance)
			{
				bIsOverridden = CurrentMaterialInstance->BasePropertyOverrides.bOverride_ShadingModel;
			}
		}
		else if (BasePropertyOverrideName.Key.IsEqual(TEXT("bOverride_DitheredLODTransition")))
		{
			TempValue = (float)CurrentMaterialInterface->IsDitheredLODTransition();
			if (CurrentMaterialInstance)
			{
				bIsOverridden = CurrentMaterialInstance->BasePropertyOverrides.bOverride_DitheredLODTransition;
			}
		}
		else if (BasePropertyOverrideName.Key.IsEqual(TEXT("bOverride_CastDynamicShadowAsMasked")))
		{
			TempValue = CurrentMaterialInterface->GetCastShadowAsMasked();
			if (CurrentMaterialInstance)
			{
				bIsOverridden = CurrentMaterialInstance->BasePropertyOverrides.bOverride_CastDynamicShadowAsMasked;
			}
		}
		else if (BasePropertyOverrideName.Key.IsEqual(TEXT("bOverride_TwoSided")))
		{
			TempValue = CurrentMaterialInterface->IsTwoSided();
			if (CurrentMaterialInstance)
			{
				bIsOverridden = CurrentMaterialInstance->BasePropertyOverrides.bOverride_TwoSided;
			}
		}
		else if (BasePropertyOverrideName.Key.IsEqual(TEXT("bOverride_bIsThinSurface")))
		{
			TempValue = CurrentMaterialInterface->IsThinSurface();
			if (CurrentMaterialInstance)
			{
				bIsOverridden = CurrentMaterialInstance->BasePropertyOverrides.bOverride_bIsThinSurface;
			}
		}
		else if (BasePropertyOverrideName.Key.IsEqual(TEXT("bOverride_OutputTranslucentVelocity")))
		{
			TempValue = CurrentMaterialInterface->IsTranslucencyWritingVelocity();
			if (CurrentMaterialInstance)
			{
				bIsOverridden = CurrentMaterialInstance->BasePropertyOverrides.bOverride_OutputTranslucentVelocity;
			}
		}
		else if (BasePropertyOverrideName.Key.IsEqual(TEXT("bOverride_bHasPixelAnimation")))
		{
			TempValue = CurrentMaterialInterface->HasPixelAnimation();
			if (CurrentMaterialInstance)
			{
				bIsOverridden = CurrentMaterialInstance->BasePropertyOverrides.bOverride_bHasPixelAnimation;
			}
		}
		else if (BasePropertyOverrideName.Key.IsEqual(TEXT("bOverride_bEnableTessellation")))
		{
			TempValue = CurrentMaterialInterface->IsTessellationEnabled();
			if (CurrentMaterialInstance)
			{
				bIsOverridden = CurrentMaterialInstance->BasePropertyOverrides.bOverride_bEnableTessellation;
			}
		}

		// Check the parent for this variable
		FAnalyzedMaterialNodePtr Parent = CurrentMaterialNode->Parent;
		if (!bIsOverridden && Parent.IsValid())
		{			
			// We shouldn't be able to get in here for the base Material
			FBasePropertyOverrideNodeRef ParentParameter = Parent->FindBasePropertyOverride(BasePropertyOverrideName.Value);

			CurrentMaterialNode->BasePropertyOverrides.Add(FBasePropertyOverrideNodeRef(
				new FBasePropertyOverrideNode(ParentParameter->ParameterName,
					ParentParameter->ParameterID,
					ParentParameter->ParameterValue,
					false,
					this->ParameterFilter)));	
		}
		else
		{
			CurrentMaterialNode->BasePropertyOverrides.Add(FBasePropertyOverrideNodeRef(
				new FBasePropertyOverrideNode(BasePropertyOverrideName.Value, BasePropertyOverrideName.Key, TempValue, bIsOverridden, this->ParameterFilter)));
		}
	}

	FMaterialLayersFunctions MaterialLayers;
	if (CurrentMaterialInterface->GetMaterialLayers(MaterialLayers))
	{
		bool bIsOverridden = false;
		if (CurrentMaterialInstance)
		{
			FMaterialLayersFunctions ParentMaterialLayers;
			const bool bParentHasLayers = CurrentMaterialInstance->Parent->GetMaterialLayers(ParentMaterialLayers);
			bIsOverridden = !bParentHasLayers || MaterialLayers != ParentMaterialLayers;
		}

		CurrentMaterialNode->MaterialLayerParameters.Add(FStaticMaterialLayerParameterNodeRef(
			new FStaticMaterialLayerParameterNode(FName(),
				MaterialLayers.GetStaticPermutationString(),
				bIsOverridden,
				this->ParameterFilter)));
	}
	
	CurrentMaterialNode->StaticSwitchParameters.Empty(StaticSwitchParameterInfo.Num());

	for (int ParameterIndex = 0; ParameterIndex < StaticSwitchParameterInfo.Num(); ++ParameterIndex)
	{
		FMaterialParameterMetadata Meta;
		bool bIsOverridden = false;
		if (CurrentMaterialInstance)
		{
			bIsOverridden = CurrentMaterialInstance->GetParameterOverrideValue(EMaterialParameterType::StaticSwitch, StaticSwitchParameterInfo[ParameterIndex], Meta);
		}
		else if(CurrentMaterial)
		{
			bIsOverridden = CurrentMaterial->GetParameterValue(EMaterialParameterType::StaticSwitch, StaticSwitchParameterInfo[ParameterIndex], Meta);
		}

		if (!IsIncludedInParameterFilter(StaticSwitchParameterInfo[ParameterIndex].Name))
		{
			continue;
		}

		if (!bIsOverridden)
		{
			// Check the parent for this variable
			FAnalyzedMaterialNodePtr Parent = CurrentMaterialNode->Parent;
			// We shouldn't be able to get in here for the base Material
			check(Parent.IsValid());

			FStaticSwitchParameterNodeRef ParentParameter = Parent->FindStaticSwitchParameter(StaticSwitchParameterInfo[ParameterIndex].Name);

			CurrentMaterialNode->StaticSwitchParameters.Add(FStaticSwitchParameterNodeRef(
				new FStaticSwitchParameterNode(ParentParameter->ParameterName,
					ParentParameter->ParameterValue,
					false,
					this->ParameterFilter)));
		}
		else
		{
			CurrentMaterialNode->StaticSwitchParameters.Add(FStaticSwitchParameterNodeRef(
				new FStaticSwitchParameterNode(StaticSwitchParameterInfo[ParameterIndex].Name, Meta.Value.AsStaticSwitch(), true, this->ParameterFilter)));
		}
	}
	
	CurrentMaterialNode->StaticComponentMaskParameters.Empty(StaticMaskParameterInfo.Num());
	
	for (int ParameterIndex = 0; ParameterIndex < StaticMaskParameterInfo.Num(); ++ParameterIndex)
	{
		if (!IsIncludedInParameterFilter(StaticMaskParameterInfo[ParameterIndex].Name))
		{
			continue;
		}

		FMaterialParameterMetadata Meta;
		bool bIsOverridden = false;
		if (CurrentMaterialInstance)
		{
			bIsOverridden = CurrentMaterialInstance->GetParameterOverrideValue(EMaterialParameterType::StaticComponentMask, StaticMaskParameterInfo[ParameterIndex], Meta);
		}
		else if (CurrentMaterial)
		{
			bIsOverridden = CurrentMaterial->GetParameterValue(EMaterialParameterType::StaticComponentMask, StaticMaskParameterInfo[ParameterIndex], Meta);
		}

		if(!bIsOverridden)
		{
			// Check the parent for this variable
			FAnalyzedMaterialNodePtr Parent = CurrentMaterialNode->Parent;
			// We shouldn't be able to get in here for the base Material
			check(Parent.IsValid());

			FStaticComponentMaskParameterNodeRef ParentParameter = Parent->FindStaticComponentMaskParameter(StaticMaskParameterInfo[ParameterIndex].Name);

			CurrentMaterialNode->StaticComponentMaskParameters.Add(FStaticComponentMaskParameterNodeRef(
				new FStaticComponentMaskParameterNode(ParentParameter->ParameterName,
					ParentParameter->R,
					ParentParameter->G,
					ParentParameter->B,
					ParentParameter->A,
					false,
					this->ParameterFilter)));
		}
		else
		{
			CurrentMaterialNode->StaticComponentMaskParameters.Add(FStaticComponentMaskParameterNodeRef(
				new FStaticComponentMaskParameterNode(StaticMaskParameterInfo[ParameterIndex].Name,
					Meta.Value.Bool[0],
					Meta.Value.Bool[1],
					Meta.Value.Bool[2],
					Meta.Value.Bool[3],
					true,
					this->ParameterFilter)));
		}
	}

	CurrentMaterialQueueIndex++;
}

bool FAnalyzeForIdenticalPermutationsAsyncTask::CreateMaterialPermutationHashForNode(const FAnalyzedMaterialNodeRef& MaterialNode, uint32& OutHash)
{
	TArray<char> ByteArray;

	bool bAnyOverrides = false;

	for (int ParameterIndex = 0; ParameterIndex < MaterialNode->BasePropertyOverrides.Num(); ++ParameterIndex)
	{
		FString FloatToHash = FString::SanitizeFloat(MaterialNode->BasePropertyOverrides[ParameterIndex]->ParameterValue);
		ByteArray.Append(TCHAR_TO_ANSI(*FloatToHash), FloatToHash.Len());
		bAnyOverrides = bAnyOverrides || MaterialNode->BasePropertyOverrides[ParameterIndex]->bOverride;
	}

	for (int ParameterIndex = 0; ParameterIndex < MaterialNode->MaterialLayerParameters.Num(); ++ParameterIndex)
	{
		ByteArray.Append(TCHAR_TO_ANSI(*MaterialNode->MaterialLayerParameters[ParameterIndex]->ParameterValue), MaterialNode->MaterialLayerParameters[ParameterIndex]->ParameterValue.Len());
		bAnyOverrides = bAnyOverrides || MaterialNode->MaterialLayerParameters[ParameterIndex]->bOverride;
	}

	for(int ParameterIndex = 0; ParameterIndex < MaterialNode->StaticSwitchParameters.Num(); ++ParameterIndex)
	{
		ByteArray.Add(MaterialNode->StaticSwitchParameters[ParameterIndex]->ParameterValue ? 1 : 0);
		bAnyOverrides = bAnyOverrides || MaterialNode->StaticSwitchParameters[ParameterIndex]->bOverride;
	}

	for(int ParameterIndex = 0; ParameterIndex < MaterialNode->StaticComponentMaskParameters.Num(); ++ParameterIndex)
	{
		FStaticComponentMaskParameterNodeRef NodeRef = MaterialNode->StaticComponentMaskParameters[ParameterIndex];
		ByteArray.Add(NodeRef->R ? 1 : 0);
		ByteArray.Add(NodeRef->G ? 1 : 0);
		ByteArray.Add(NodeRef->B ? 1 : 0);
		ByteArray.Add(NodeRef->A ? 1 : 0);
		bAnyOverrides = bAnyOverrides || NodeRef->bOverride;
	}

	OutHash = CityHash32(ByteArray.GetData(), ByteArray.Num());

	return bAnyOverrides;
}

void FAnalyzeForIdenticalPermutationsAsyncTask::DoWork()
{
	for(int i = 0; i < MaterialQueue.Num(); ++i)
	{
		FAnalyzedMaterialNodeRef CurrentMaterialNode = MaterialQueue[i];

		MaterialQueue.Append(CurrentMaterialNode->GetChildNodes());

		uint32 MaterialPermutationHash = 0;

		if(CreateMaterialPermutationHashForNode(CurrentMaterialNode, MaterialPermutationHash))
		{
			MaterialPermutationHashToMaterialObjectPath.FindOrAdd(MaterialPermutationHash).Add(CurrentMaterialNode->ObjectPath);
		}
	}

	GatherSuggestions();
}

void FAnalyzeForIdenticalPermutationsAsyncTask::GatherSuggestions()
{
	Suggestions.Empty();
	for (TPair<uint32, TArray<FSoftObjectPath>>& IdenticalPermutations : MaterialPermutationHashToMaterialObjectPath)
	{
		if (IdenticalPermutations.Value.Num() > 1)
		{
			TArray<FString> AllNames;
			AssetCount = IdenticalPermutations.Value.Num();
			for (int i = 0; i < IdenticalPermutations.Value.Num(); ++i)
			{
				FString PermutationString = IdenticalPermutations.Value[i].ToString();
				AllNames.Add(PermutationString);
			}

			FPermutationSuggestionData NewData = FPermutationSuggestionData(FText::Format(LOCTEXT("IdenticalStaticPermutationSuggestions", "The following {0} materials all have identical static parameter permutations."),
				FText::AsNumber(AssetCount)),
				AllNames);

			Suggestions.Add
				(
					AssetCount,
					NewData
				);
		}
	}
}


#undef LOCTEXT_NAMESPACE
