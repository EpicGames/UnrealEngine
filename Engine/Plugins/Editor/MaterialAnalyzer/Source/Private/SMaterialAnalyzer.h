// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"
#include "Materials/MaterialParameters.h"
#include "Widgets/SCompoundWidget.h"
#include "AnalyzedMaterialNode.h"

class ICollectionContainer;
class ITableRow;
class SDockTab;
class SScrollBox;
class STableViewBase;
class STextBlock;
class SThrobber;
class UMaterialInterface;
template <typename ItemType> class STreeView;

struct FBuildBasicMaterialTreeAsyncTask : FNonAbandonableTask
{
public:
	/** File data loaded for the async read */
	TArray<FAnalyzedMaterialNodeRef>& MaterialTreeRoot;
	
	TArray<FAssetData> AssetDataToAnalyze;

	FString ParameterFilter;

	/** Initializes the variables needed to load and verify the data */
	FBuildBasicMaterialTreeAsyncTask(TArray<FAnalyzedMaterialNodeRef>& InMaterialTreeRoot, const TArray<FAssetData>& InAssetDataToAnalyze, const FString& InParameterFilter = TEXT(""))
		: MaterialTreeRoot(InMaterialTreeRoot)
		, AssetDataToAnalyze(InAssetDataToAnalyze)
		, ParameterFilter(InParameterFilter)
	{
	}

	FAnalyzedMaterialNodePtr FindOrMakeBranchNode(FAnalyzedMaterialNodePtr ParentNode, const FAssetData* ChildData);

	/**
	* Loads and hashes the file data. Empties the data if the hash check fails
	*/
	void DoWork();

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FBuildBasicMaterialTreeAsyncTask, STATGROUP_ThreadPoolAsyncTasks);
	}
};

struct FAnalyzeMaterialTreeAsyncTask
{
public:
	FAnalyzedMaterialNodeRef MaterialTreeRoot;
	const TArray<FAssetData>& AssetDataToAnalyze;

	TArray<FAnalyzedMaterialNodeRef> MaterialQueue;

	int32 CurrentMaterialQueueIndex;

	FAnalyzedMaterialNodeRef CurrentMaterialNode;
	UMaterialInterface* CurrentMaterialInterface;

	TArray<FMaterialParameterInfo> BasePropertyOverrideInfo;

	TArray<FMaterialParameterInfo> StaticSwitchParameterInfo;
	TArray<FGuid> StaticSwitchGuids;

	TArray<FMaterialParameterInfo> StaticMaskParameterInfo;
	TArray<FGuid> StaticMaskGuids;

	FText ParameterFilter;


	FAnalyzeMaterialTreeAsyncTask(FAnalyzedMaterialNodeRef InMaterialTreeRoot, const TArray<FAssetData>& InAssetDataToAnalyze, const FText& InParameterFilter = FText::GetEmpty()):
		MaterialTreeRoot(InMaterialTreeRoot),
		AssetDataToAnalyze(InAssetDataToAnalyze),
		CurrentMaterialQueueIndex(0),
		CurrentMaterialNode(InMaterialTreeRoot),
		ParameterFilter(InParameterFilter)
	{
		MaterialQueue.Empty(MaterialTreeRoot->TotalNumberOfChildren());
		MaterialQueue.Add(MaterialTreeRoot);
		LoadNextMaterial();
	}

	bool LoadNextMaterial();

	void DoWork();

	// need to make abandon-able eventually
	bool CanAbandon()
	{
		return false;
	}
	void Abandon()
	{
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAnalyzeMaterialTreeAsyncTask, STATGROUP_ThreadPoolAsyncTasks);
	}
};

struct FPermutationSuggestionData
{
	FText Header;
	TArray<FString> Materials;

	FPermutationSuggestionData(const FText& InHeader, TArray<FString> InMaterials)
		: Header(InHeader)
		, Materials(InMaterials)
	{
	}

};

struct FPermutationSuggestionView
{
	FText Header;
	TArray<TSharedPtr<FPermutationSuggestionView>> Children;
};

struct FAnalyzeForSuggestions
{

public:
	TMultiMap<int32, FPermutationSuggestionData> GetSuggestions()
	{
		return Suggestions;
	}

protected:
	FAnalyzeForSuggestions()
		:Suggestions(TMultiMap<int32, FPermutationSuggestionData>())
	{
	}

	virtual ~FAnalyzeForSuggestions() = default;

	TMultiMap<int32, FPermutationSuggestionData> Suggestions;

	virtual void GatherSuggestions() = 0;

};

struct FAnalyzeForIdenticalPermutationsAsyncTask : FAnalyzeForSuggestions
{
public:
	virtual ~FAnalyzeForIdenticalPermutationsAsyncTask() = default;

	FAnalyzedMaterialNodeRef MaterialTreeRoot;

	TArray<FAnalyzedMaterialNodeRef> MaterialQueue;

	TMap<uint32, TArray<FSoftObjectPath>> MaterialPermutationHashToMaterialObjectPath;

	int32 AssetCount;

	FAnalyzeForIdenticalPermutationsAsyncTask(FAnalyzedMaterialNodeRef InMaterialTreeRoot) :
		MaterialTreeRoot(InMaterialTreeRoot)
	{
		MaterialQueue.Empty(MaterialTreeRoot->TotalNumberOfChildren());
		MaterialQueue.Add(MaterialTreeRoot);
	}

	bool CreateMaterialPermutationHashForNode(const FAnalyzedMaterialNodeRef& MaterialNode, uint32& OutHash);
	void DoWork();

	// need to make abandon-able eventually
	bool CanAbandon()
	{
		return false;
	}
	void Abandon()
	{
	}

	virtual void GatherSuggestions() override;

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAnalyzeForIdenticalPermutationsAsyncTask, STATGROUP_ThreadPoolAsyncTasks);
	}
};

class SMaterialAnalyzer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMaterialAnalyzer)
	{ }
	SLATE_END_ARGS()

	typedef STreeView<FAnalyzedMaterialNodeRef> SAnalyzedMaterialTree;

public:
	SMaterialAnalyzer();
	virtual ~SMaterialAnalyzer();

	/**
	* Constructs the application.
	*
	* @param InArgs The Slate argument list.
	* @param ConstructUnderMajorTab The major tab which will contain the session front-end.
	* @param ConstructUnderWindow The window in which this widget is being constructed.
	* @param InStyleSet The style set to use.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	TSharedRef< ITableRow > OnGenerateSuggestionRow(TSharedPtr<FPermutationSuggestionView> Item, const TSharedRef< STableViewBase >& OwnerTable);
	EVisibility ShouldShowAdvancedRecommendations(TSharedPtr<FPermutationSuggestionView> Item) const;
	void OnAssetAdded(const FAssetData& InAssetData);

protected:

	TArray<FAssetData> AssetDataArray;
	TArray<FAssetData> RecentlyAddedAssetData;
	TArray<FAssetData> RecentlyRemovedAssetData;

	TSharedPtr<SAnalyzedMaterialTree> MaterialTree;
	TSharedPtr<STextBlock> StatusBox;
	TSharedPtr<SScrollBox> SuggestionsBox;
	TSharedPtr<SThrobber> StatusThrobber;

	TArray<FAnalyzedMaterialNodeRef> AllMaterialTreeRoots;
	TArray<FAnalyzedMaterialNodeRef> MaterialTreeRoot;

	/** The tree view widget */
	TSharedPtr<STreeView<TSharedPtr<FPermutationSuggestionView>>> SuggestionsTree;
	TArray<TSharedPtr<FPermutationSuggestionView>> SuggestionDataArray;

	FAsyncTask<FBuildBasicMaterialTreeAsyncTask>* BuildBaseMaterialTreeTask;
	FAsyncTask<FAnalyzeMaterialTreeAsyncTask>* AnalyzeTreeTask;
	FAsyncTask<FAnalyzeForIdenticalPermutationsAsyncTask>* AnalyzeForIdenticalPermutationsTask;

	FAssetData CurrentlySelectedAsset;

	bool bWaitingForAssetRegistryLoad;

	void OnAssetSelected(const FAssetData& AssetData);
	void OnParameterFilterChanged(const FText& Filter, const ETextCommit::Type InTextAction);

	FReply OnExportAnalyzedMaterialToCSV();

	void BuildBasicMaterialTree();

	int32 GetTotalNumberOfMaterialNodes();

	void SetupAssetRegistryCallbacks();

	void OnGetSuggestionChildren(TSharedPtr<FPermutationSuggestionView> InParent, TArray< TSharedPtr<FPermutationSuggestionView> >& OutChildren);
	FReply CreateLocalSuggestionCollectionClicked(TSharedPtr<FPermutationSuggestionView> InSuggestion);
	void CreateLocalSuggestionCollection(ICollectionContainer& InCollectionContainer, const FPermutationSuggestionView& InSuggestion);
	void StartAsyncWork(const FText& WorkText);

	void AsyncWorkFinished(const FText& CompleteText);

	FString GetCurrentAssetPath() const;

private:

	/** Callback for generating a row in the reflector tree view. */
	TSharedRef<ITableRow> HandleReflectorTreeGenerateRow(FAnalyzedMaterialNodeRef InReflectorNode, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback for getting the child items of the given reflector tree node. */
	void HandleReflectorTreeGetChildren(FAnalyzedMaterialNodeRef InReflectorNode, TArray<FAnalyzedMaterialNodeRef>& OutChildren);

	/**
	 * Called to recursively expand/collapse the children of the given item
	 *
	 * @param InTreeNode		The node that was expanded or collapsed
	 * @param bIsItemExpanded	True if the item is expanded, false if it is collapsed
	 */
	void HandleReflectorTreeRecursiveExpansion(FAnalyzedMaterialNodeRef InTreeNode, bool bIsItemExpanded);

	void UpdateViewForSelectedAsset();

	bool IsMaterialSelectionAllowed() const
	{
		// Don't allow to change selected material while async work is operating on currently selected asset.
		return !bIsAsyncWorkInProgress;
	}

private:
	bool bIsAsyncWorkInProgress = false;
	FText ParameterFilter;
	bool bHasParameterFilterChanged = false;
};
